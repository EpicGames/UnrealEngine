// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class CurveTableEditor : ModuleRules
{
	public CurveTableEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.Add("LevelEditor");

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core", 
				"CoreUObject", 
                "CurveEditor",
				"EditorFramework",
				"Engine", 
				"InputCore",
				"Slate",
				"SlateCore",
				"UnrealEd",
				"EditorWidgets",
				"ToolWidgets",
			}
		);

		DynamicallyLoadedModuleNames.Add("WorkspaceMenuStructure");
	}
}
