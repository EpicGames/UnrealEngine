// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeCurveTableEditor.h"

#include "Delegates/Delegate.h"
#include "DetailsViewArgs.h"
#include "IDetailsView.h"
#include "Internationalization/Internationalization.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"

class UCurveTable;

#define LOCTEXT_NAMESPACE "CompositeCurveTableEditor"

const FName FCompositeCurveTableEditor::PropertiesTabId("CompositeDataTableEditor_Properties");


void FCompositeCurveTableEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FCurveTableEditor::RegisterTabSpawners(InTabManager);

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	CreateAndRegisterPropertiesTab(InTabManager);
}


void FCompositeCurveTableEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FCurveTableEditor::UnregisterTabSpawners(InTabManager);

	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(PropertiesTabId);

	DetailsView.Reset();
}

void FCompositeCurveTableEditor::CreateAndRegisterPropertiesTab(const TSharedRef<class FTabManager>& InTabManager)
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;

	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	InTabManager->RegisterTabSpawner(PropertiesTabId, FOnSpawnTab::CreateSP(this, &FCompositeCurveTableEditor::SpawnTab_Properties))
		.SetDisplayName(LOCTEXT("PropertiesTab", "Properties"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef());
}

TSharedRef<SDockTab> FCompositeCurveTableEditor::SpawnTab_Properties(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == PropertiesTabId);

	return SNew(SDockTab)
		.Label(LOCTEXT("PropertiesTitle", "Properties"))
		.TabColorScale(GetTabColorScale())
		[
			DetailsView.ToSharedRef()
		];
}

void FCompositeCurveTableEditor::InitCurveTableEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UCurveTable* Table )
{
	FCurveTableEditor::InitCurveTableEditor(Mode, InitToolkitHost, Table);

	if (DetailsView.IsValid())
	{
		// Make sure details window is pointing to our object
		DetailsView->SetObject(GetEditingObject());
	}
}

TSharedRef< FTabManager::FLayout > FCompositeCurveTableEditor::InitCurveTableLayout()
{
	return FTabManager::NewLayout("Standalone_CompositeCurveTableEditor_temp_Layout3")
	->AddArea
	(
		FTabManager::NewPrimaryArea()->SetOrientation(Orient_Horizontal)
		->Split
		(
			FTabManager::NewSplitter()
			->SetOrientation(Orient_Vertical)
			->SetSizeCoefficient(0.3f)
			->Split
			(
				FTabManager::NewStack()
				->SetHideTabWell(true)
				->AddTab(PropertiesTabId, ETabState::OpenedTab)
			)
		)
		->Split
		(
			FTabManager::NewSplitter()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewStack()
				->AddTab(CurveTableTabId, ETabState::OpenedTab)
			)
		)
	);
}

FName FCompositeCurveTableEditor::GetToolkitFName() const
{
	return FName("CompositeCurveTableEditor");
}

FText FCompositeCurveTableEditor::GetBaseToolkitName() const
{
	return LOCTEXT( "AppLabel", "Composite CurveTable Editor" );
}

#undef LOCTEXT_NAMESPACE
