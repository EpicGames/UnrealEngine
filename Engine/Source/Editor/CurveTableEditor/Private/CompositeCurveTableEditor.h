// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveTableEditor.h"
#include "Framework/Docking/TabManager.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Toolkits/IToolkit.h"
#include "UObject/NameTypes.h"

class SDockTab;
class UCurveTable;


/** Viewer/editor for a CurveTable */
class FCompositeCurveTableEditor :
	public FCurveTableEditor
{
	friend class SCurveTableListViewRow;
	friend class SCurveTableCurveViewRow;

public:
	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

	virtual void InitCurveTableEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UCurveTable* Table) override;

	void CreateAndRegisterPropertiesTab(const TSharedRef<class FTabManager>& InTabManager);

	/**	Spawns the tab with the details view inside */
	TSharedRef<SDockTab> SpawnTab_Properties(const FSpawnTabArgs& Args);

	/** IToolkit interface */
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;

protected:
	virtual TSharedRef< FTabManager::FLayout > InitCurveTableLayout() override;

	virtual bool ShouldCreateDefaultStandaloneMenu() const override { return true; }
	virtual bool ShouldCreateDefaultToolbar() const override { return true; }

private:
	/** Details view */
	TSharedPtr< class IDetailsView > DetailsView;

	static const FName PropertiesTabId;
};
