// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveTableEditor.h"

#include "Containers/ArrayView.h"
#include "CurveEditor.h"
#include "CurveModel.h"
#include "CurveTableEditorCommands.h"
#include "CurveTableEditorHandle.h"
#include "CurveTableEditorModule.h"
#include "Curves/KeyHandle.h"
#include "Curves/SimpleCurve.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "EditorReimportHandler.h"
#include "Engine/CompositeCurveTable.h"
#include "Engine/CurveTable.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/Text/TextLayout.h"
#include "Framework/Views/ITypedTableView.h"
#include "ICurveEditorModule.h"
#include "Internationalization/Internationalization.h"
#include "Layout/BasicLayoutWidgetSlot.h"
#include "Layout/Margin.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "RealCurveModel.h"
#include "Rendering/SlateRenderer.h"
#include "RichCurveEditorModel.h"
#include "SCurveEditorPanel.h"
#include "SPositiveActionButton.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"
#include "Templates/Casts.h"
#include "Templates/Tuple.h"
#include "Templates/UniquePtr.h"
#include "Textures/SlateIcon.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Tree/CurveEditorTree.h"
#include "Tree/CurveEditorTreeFilter.h"
#include "Tree/CurveEditorTreeTraits.h"
#include "Tree/ICurveEditorTreeItem.h"
#include "Tree/SCurveEditorTree.h"
#include "Tree/SCurveEditorTreePin.h"
#include "Tree/SCurveEditorTreeSelect.h"
#include "Tree/SCurveEditorTreeTextFilter.h"
#include "Types/SlateStructs.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Views/SListView.h"

class ITableRow;
class SWidget;
class UObject;
struct FRichCurve;

 
#define LOCTEXT_NAMESPACE "CurveTableEditor"

const FName FCurveTableEditor::CurveTableTabId("CurveTableEditor_CurveTable");

struct FCurveTableEditorColumnHeaderData
{
	/** Unique ID used to identify this column */
	FName ColumnId;

	/** Display name of this column */
	FText DisplayName;

	/** The calculated width of this column taking into account the cell data for each row */
	float DesiredColumnWidth;

	/** The evaluated key time **/
	float KeyTime;
};

namespace {

		FName MakeUniqueCurveName( UCurveTable* Table )
		{
				check(Table != nullptr);

				int incr = 0;	
				FName TestName = FName("Curve", incr);

				const TMap<FName, FRealCurve*>& RowMap = Table->GetRowMap();

				while (RowMap.Contains(TestName))
				{
						TestName = FName("Curve", ++incr);
				}

				return TestName;
		}
}

/*
* FCurveTableEditorItem
*
*  FCurveTableEditorItem uses and extends the CurveEditorTreeItem to be used in both our TableView and the CurveEditorTree.
*  The added GenerateTableViewCell handles the table columns unknown to the standard CurveEditorTree.
*
*/ 
class FCurveTableEditorItem : public ICurveEditorTreeItem,  public TSharedFromThis<FCurveTableEditorItem>
{

  	struct CachedKeyInfo
  	{
  		CachedKeyInfo(FKeyHandle& InKeyHandle, FText InDisplayValue) :
  		KeyHandle(InKeyHandle)
  		, DisplayValue(InDisplayValue) {}

  		FKeyHandle KeyHandle;

  		FText DisplayValue;	
  	};

  public: 
	FCurveTableEditorItem (TWeakPtr<FCurveTableEditor> InCurveTableEditor, const FCurveEditorTreeItemID& InTreeID, const FName& InRowId, FCurveTableEditorHandle InRowHandle, const TArray<FCurveTableEditorColumnHeaderDataPtr>& InColumns)
		: CurveTableEditor(InCurveTableEditor)
		, TreeID(InTreeID)
		, RowId(InRowId)
		, RowHandle(InRowHandle)
		, Columns(InColumns)
	{
		DisplayName = FText::FromName(InRowId);

		CacheKeys();
	}

	TSharedPtr<SWidget> GenerateCurveEditorTreeWidget(const FName& InColumnName, TWeakPtr<FCurveEditor> InCurveEditor, FCurveEditorTreeItemID InTreeItemID, const TSharedRef<ITableRow>& InTableRow) override
	{
		if (InColumnName == ColumnNames.Label)
		{
			return SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.Padding(FMargin(4.f))
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				.AutoWidth()
				[
					SAssignNew(InlineRenameWidget, SInlineEditableTextBlock)
					.Text(DisplayName)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.MaximumLength(NAME_SIZE-1)
					.OnTextCommitted(this, &FCurveTableEditorItem::HandleNameCommitted)
					.OnVerifyTextChanged(this, &FCurveTableEditorItem::VerifyNameChanged)
				];
		}
		else if (InColumnName == ColumnNames.SelectHeader)
		{
			return SNew(SCurveEditorTreeSelect, InCurveEditor, InTreeItemID, InTableRow);
		}
		else if (InColumnName == ColumnNames.PinHeader)
		{
			return SNew(SCurveEditorTreePin, InCurveEditor, InTreeItemID, InTableRow);
		}

		return GenerateTableViewCell(InColumnName, InCurveEditor, InTreeItemID, InTableRow);
	}

	TSharedPtr<SWidget> GenerateTableViewCell(const FName& InColumnId, TWeakPtr<FCurveEditor> InCurveEditor, FCurveEditorTreeItemID InTreeItemID, const TSharedRef<ITableRow>& InTableRow)
	{
		if (!RowHandle.HasRichCurves())
		{
			FRealCurve* Curve = RowHandle.GetCurve();
			FKeyHandle& KeyHandle = CellDataMap[InColumnId].KeyHandle;

			return SNew(SNumericEntryBox<float>)
				.EditableTextBoxStyle( &FAppStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("CurveTableEditor.Cell.Text") )
				.Value_Lambda([this, KeyHandle] () { 
					if (FRealCurve* Curve = RowHandle.GetCurve())
					{
						return Curve->GetKeyValue(KeyHandle); 
					}
					return 0.0f;
				})
				.OnValueChanged_Lambda([this, KeyHandle] (float NewValue) 
				{
					if (FRealCurve* Curve = RowHandle.GetCurve())
					{
						FScopedTransaction Transaction(LOCTEXT("SetKeyValues", "Set Key Values"));
						RowHandle.ModifyOwner();
						Curve->SetKeyValue(KeyHandle, NewValue);
					}
				})
				.Justification(ETextJustify::Right)
			;
		}
		return SNullWidget::NullWidget;
	}

	void CreateCurveModels(TArray<TUniquePtr<FCurveModel>>& OutCurveModels) override
	{
		if (RowHandle.HasRichCurves())
		{
			if (FRichCurve* RichCurve = RowHandle.GetRichCurve())
			{
				const UCurveTable* Table = RowHandle.CurveTable.Get();
				UCurveTable* RawTable = const_cast<UCurveTable*>(Table);

				TUniquePtr<FRichCurveEditorModelRaw> NewCurve = MakeUnique<FRichCurveEditorModelRaw>(RichCurve, RawTable);
				NewCurve->SetShortDisplayName(DisplayName);
				NewCurve->SetColor(FStyleColors::AccentOrange.GetSpecifiedColor());
				OutCurveModels.Add(MoveTemp(NewCurve));
			}
		}
		else
		{
			const UCurveTable* Table = RowHandle.CurveTable.Get();
			UCurveTable* RawTable = const_cast<UCurveTable*>(Table);

			TUniquePtr<FRealCurveModel> NewCurveModel = MakeUnique<FRealCurveModel>(RowHandle, RawTable);
			NewCurveModel->SetShortDisplayName(DisplayName);

			OutCurveModels.Add(MoveTemp(NewCurveModel));
		}
	}

	bool PassesFilter(const FCurveEditorTreeFilter* InFilter) const override
	{
		if (InFilter->GetType() == ECurveEditorTreeFilterType::Text)
		{
			FString DisplayNameAsString = DisplayName.ToString();

			const FCurveEditorTreeTextFilter* Filter = static_cast<const FCurveEditorTreeTextFilter*>(InFilter);
			for (const FCurveEditorTreeTextFilterTerm& Term : Filter->GetTerms())
			{
				if (!Term.Match(DisplayNameAsString).IsTotalMatch())
				{
					return false;
				}
			}

			return true;
		}

		return false;
	}

	void CacheKeys()
	{
		if (!RowHandle.HasRichCurves())
		{
			if (FRealCurve* Curve = RowHandle.GetCurve())
			{	
				for (auto Col : Columns)
				{
					FKeyHandle KeyHandle = Curve->FindKey(Col->KeyTime);
					float KeyValue = Curve->GetKeyValue(KeyHandle);

					CellDataMap.Add(Col->ColumnId, CachedKeyInfo(KeyHandle, FText::AsNumber(KeyValue))); 
				}
			}
		}
	}

	void EnterRenameMode()
	{
		InlineRenameWidget->EnterEditingMode();
	}

	bool VerifyNameChanged(const FText& InText, FText& OutErrorMessage)
	{
		FName CheckName = FName(*InText.ToString());
		if (CheckName == RowId)
		{
			return true;	
		}

		if (RowHandle.CurveTable.IsValid())
		{
			UCurveTable* Table = RowHandle.CurveTable.Get();
			const TMap<FName, FRealCurve*>& RowMap = Table->GetRowMap();
			if (RowMap.Contains(CheckName))
			{

				OutErrorMessage = LOCTEXT("NameAlreadyUsed", "Row Names Must Be Unique");
				return false;
			}
			return true;
		}
		return false;
	}

	void HandleNameCommitted(const FText& CommittedText, ETextCommit::Type CommitInfo)
	{
		if (CommitInfo == ETextCommit::OnEnter)
		{
			TSharedPtr<FCurveTableEditor> TableEditorPtr = CurveTableEditor.Pin();
			if (TableEditorPtr != nullptr)
			{
				FName OldName = RowId;
				FName NewName = *CommittedText.ToString();

				DisplayName = CommittedText;
				InlineRenameWidget->SetText(DisplayName);

				RowHandle.RowName = NewName;
				RowId = NewName;

				TableEditorPtr->HandleCurveRename(TreeID, OldName, NewName);

				TSharedPtr<FCurveEditor> CurveEditor = TableEditorPtr->GetCurveEditor();
				FCurveEditorTreeItem& TreeItem = CurveEditor->GetTreeItem(TreeID);
				for (FCurveModelID ModelID : TreeItem.GetCurves())
				{
					if (FCurveModel* CurveModel = CurveEditor->FindCurve(ModelID))
					{
						CurveModel->SetShortDisplayName(DisplayName);
					}
				}
			}
		}
	}

	/** Hold onto a weak ptr to the CurveTableEditor specifically for deleting and renaming  */
	TWeakPtr<FCurveTableEditor> CurveTableEditor;

	/** The CurveEditor's Unique ID for the TreeItem this item is attached to (SetStrongItem) */
	FCurveEditorTreeItemID TreeID;

	/** Unique ID used to identify this row */
	FName RowId;

	/** Display name of this row */
	FText DisplayName;

	/** Array corresponding to each cell in this row */
	TMap<FName, CachedKeyInfo> CellDataMap;

	/** Handle to the row */
	FCurveTableEditorHandle RowHandle;

	/** A Reference to the available columns in the TableView */
	const TArray<FCurveTableEditorColumnHeaderDataPtr>& Columns;

	/** Inline editable text box for renaming */
	TSharedPtr<SInlineEditableTextBlock> InlineRenameWidget;

};


void FCurveTableEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_CurveTableEditor", "Curve Table Editor"));

	InTabManager->RegisterTabSpawner( CurveTableTabId, FOnSpawnTab::CreateSP(this, &FCurveTableEditor::SpawnTab_CurveTable) )
		.SetDisplayName( LOCTEXT("CurveTableTab", "Curve Table") )
		.SetGroup( WorkspaceMenuCategory.ToSharedRef() );
}


void FCurveTableEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	InTabManager->UnregisterTabSpawner( CurveTableTabId );
}


FCurveTableEditor::~FCurveTableEditor()
{
	FReimportManager::Instance()->OnPostReimport().RemoveAll(this);
	
	if (UCurveTable* CurveTable = GetCurveTable())
	{
		CurveTable->OnCurveTableChanged().RemoveAll(this);
	}
}


void FCurveTableEditor::InitCurveTableEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UCurveTable* Table )
{
	const TSharedRef< FTabManager::FLayout > StandaloneDefaultLayout = InitCurveTableLayout();

	FAssetEditorToolkit::InitAssetEditor( Mode, InitToolkitHost, FCurveTableEditorModule::CurveTableEditorAppIdentifier, StandaloneDefaultLayout, ShouldCreateDefaultStandaloneMenu(), ShouldCreateDefaultToolbar(), Table );
	
	BindCommands();
	ExtendMenu();
	ExtendToolbar();
	RegenerateMenusAndToolbars();

	FReimportManager::Instance()->OnPostReimport().AddSP(this, &FCurveTableEditor::OnPostReimport);	
	
	if (Table)
	{
		Table->OnCurveTableChanged().AddSP(this, &FCurveTableEditor::RefreshTableRows);
	}

	GEditor->RegisterForUndo(this);
}

TSharedRef< FTabManager::FLayout > FCurveTableEditor::InitCurveTableLayout()
{
	return FTabManager::NewLayout("Standalone_CurveTableEditor_Layout_v1.1")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->Split
			(
				FTabManager::NewStack()
				->AddTab(CurveTableTabId, ETabState::OpenedTab)
				->SetHideTabWell(true)
			)
		);
}

void FCurveTableEditor::BindCommands()
{
	FCurveTableEditorCommands::Register();

	ToolkitCommands->MapAction(FGenericCommands::Get().Undo,   FExecuteAction::CreateLambda([]{ GEditor->UndoTransaction(); }));
	ToolkitCommands->MapAction(FGenericCommands::Get().Redo,   FExecuteAction::CreateLambda([]{ GEditor->RedoTransaction(); }));

	ToolkitCommands->MapAction(
		FCurveTableEditorCommands::Get().CurveViewToggle,
		FExecuteAction::CreateSP(this, &FCurveTableEditor::ToggleViewMode),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FCurveTableEditor::IsCurveViewChecked)
	);

	ToolkitCommands->MapAction(
		FCurveTableEditorCommands::Get().AppendKeyColumn,
		FExecuteAction::CreateSP(this, &FCurveTableEditor::OnAddNewKeyColumn)
	);

	ToolkitCommands->MapAction(
		FCurveTableEditorCommands::Get().RenameSelectedCurve,
		FExecuteAction::CreateSP(this, &FCurveTableEditor::OnRenameCurve)
	);


	ToolkitCommands->MapAction(
		FCurveTableEditorCommands::Get().DeleteSelectedCurves,
		FExecuteAction::CreateSP(this, &FCurveTableEditor::OnDeleteCurves)
	);

}

bool FCurveTableEditor::IsReadOnly() const
{
	/* Currently, the only read-only tables are composite curve tables */
	return GetCurveTable()->IsA<UCompositeCurveTable>();
}

void FCurveTableEditor::ExtendMenu()
{
	MenuExtender = MakeShareable(new FExtender);

	struct Local
	{
		static void ExtendMenu(FMenuBuilder& MenuBuilder)
		{
			MenuBuilder.BeginSection("CurveTableEditor", LOCTEXT("CurveTableEditor", "Curve Table"));
			{
				MenuBuilder.AddMenuEntry(FCurveTableEditorCommands::Get().CurveViewToggle);
			}
			MenuBuilder.EndSection();
			}
	};

	MenuExtender->AddMenuExtension(
		"WindowLayout",
		EExtensionHook::After,
		GetToolkitCommands(),
		FMenuExtensionDelegate::CreateStatic(&Local::ExtendMenu)
	);

	AddMenuExtender(MenuExtender);

	FCurveTableEditorModule& CurveTableEditorModule = FModuleManager::LoadModuleChecked<FCurveTableEditorModule>("CurveTableEditor");
	AddMenuExtender(CurveTableEditorModule.GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));
}

void FCurveTableEditor::ExtendToolbar()
{
	ToolbarExtender = MakeShareable(new FExtender);

	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateLambda([this](FToolBarBuilder& ParentToolbarBuilder)
		{
			ParentToolbarBuilder.BeginSection("CurveTable");

			ParentToolbarBuilder.AddToolBarButton(
				FUIAction(FExecuteAction::CreateSP(this, &FCurveTableEditor::Reimport_Execute, GetEditingObject())),
				NAME_None,
				FText::GetEmpty(),
				LOCTEXT("Reimport_Tooltip", "Reimport the Curve Table from the source file.  All changes will be lost.  This action cannot be undone."),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Toolbar.Reimport")
			);

			bool HasRichCurves = GetCurveTable()->HasRichCurves();
			ParentToolbarBuilder.AddWidget(
				SNew(SSegmentedControl<ECurveTableViewMode>)
				.Visibility(HasRichCurves ? EVisibility::Collapsed : EVisibility::Visible)
				.OnValueChanged_Lambda([this] (ECurveTableViewMode InMode) {if (InMode != GetViewMode()) ToggleViewMode();  } )
				.Value(this, &FCurveTableEditor::GetViewMode)

				+SSegmentedControl<ECurveTableViewMode>::Slot(ECurveTableViewMode::CurveTable)
			    .Icon(FAppStyle::Get().GetBrush("CurveTableEditor.CurveView"))

				+SSegmentedControl<ECurveTableViewMode>::Slot(ECurveTableViewMode::Grid)
			    .Icon(FAppStyle::Get().GetBrush("CurveTableEditor.TableView"))
			);

			if (!IsReadOnly())
			{
				ParentToolbarBuilder.AddToolBarButton(
				FCurveTableEditorCommands::Get().AppendKeyColumn,
				NAME_None, 
				FText::GetEmpty(),
				TAttribute<FText>(), 
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Sequencer.KeyTriangle20"));
			}
			
			ParentToolbarBuilder.EndSection();
		})
	);

	AddToolbarExtender(ToolbarExtender);
}

FName FCurveTableEditor::GetToolkitFName() const
{
	return FName("CurveTableEditor");
}

FText FCurveTableEditor::GetBaseToolkitName() const
{
	return LOCTEXT( "AppLabel", "CurveTable Editor" );
}

FString FCurveTableEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "CurveTable ").ToString();
}

FLinearColor FCurveTableEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor( 0.0f, 0.0f, 0.2f, 0.5f );
}

void FCurveTableEditor::PreChange(const UCurveTable* Changed, FCurveTableEditorUtils::ECurveTableChangeInfo Info)
{
}

void FCurveTableEditor::PostUndo(bool bSuccess)
{
	RefreshCachedCurveTable();
}

void FCurveTableEditor::PostRedo(bool bSuccess)
{
	RefreshCachedCurveTable();
}

void FCurveTableEditor::PostChange(const UCurveTable* Changed, FCurveTableEditorUtils::ECurveTableChangeInfo Info)
{
	const UCurveTable* Table = GetCurveTable();
	if (Changed == Table)
	{
		HandlePostChange();
	}
}

UCurveTable* FCurveTableEditor::GetCurveTable() const
{
	return Cast<UCurveTable>(GetEditingObject());
}

void FCurveTableEditor::HandlePostChange()
{
	RefreshCachedCurveTable();
}

TSharedRef<SDockTab> FCurveTableEditor::SpawnTab_CurveTable( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId().TabType == CurveTableTabId );

	bUpdatingTableViewSelection = false;

	bool bTableIsReadOnly = IsReadOnly();

	TSharedRef<SScrollBar> VerticalScrollBar = SNew(SScrollBar)
		.Orientation(Orient_Vertical);

	ColumnNamesHeaderRow = SNew(SHeaderRow)
		.Visibility(this, &FCurveTableEditor::GetTableViewControlsVisibility);

	CurveEditor = MakeShared<FCurveEditor>();

	FCurveEditorInitParams CurveEditorInitParams;
	CurveEditor->InitCurveEditor(CurveEditorInitParams);

	// We want this editor to handle undo, not the CurveEditor because
	// the PostUndo fixes up the selection and in the case of a CurveTable,
	// the curves have been rebuilt on undo and thus need special handling to restore the selection
	GEditor->UnregisterForUndo(CurveEditor.Get());


	CurveEditorTree = SNew(SCurveEditorTree, CurveEditor.ToSharedRef())
		.OnTreeViewScrolled(this, &FCurveTableEditor::OnCurveTreeViewScrolled)
		.OnMouseButtonDoubleClick(this, &FCurveTableEditor::OnRequestCurveRename)
		.OnContextMenuOpening(this, &FCurveTableEditor::OnOpenCurveMenu);

	TSharedRef<SCurveEditorPanel> CurveEditorPanel = SNew(SCurveEditorPanel, CurveEditor.ToSharedRef());

	TableView = SNew(SListView<FCurveEditorTreeItemID>)

		.IsEnabled(!bTableIsReadOnly)
		.ListItemsSource(&EmptyItems)
		.OnListViewScrolled(this, &FCurveTableEditor::OnTableViewScrolled)
		.HeaderRow(ColumnNamesHeaderRow)
		.OnGenerateRow(CurveEditorTree.Get(), &SCurveEditorTree::GenerateRow)
		.ExternalScrollbar(VerticalScrollBar)
		.SelectionMode(ESelectionMode::Multi)
		.OnSelectionChanged_Lambda(
			[this](TListTypeTraits<FCurveEditorTreeItemID>::NullableType InItemID, ESelectInfo::Type Type)
			{
				this->OnTableViewSelectionChanged(InItemID, Type);
			}
		);

	CurveEditor->GetTree()->Events.OnItemsChanged.AddSP(this, &FCurveTableEditor::RefreshTableRows);
	CurveEditor->GetTree()->Events.OnSelectionChanged.AddSP(this, &FCurveTableEditor::RefreshTableRowsSelection);

	ViewMode = GetCurveTable()->HasRichCurves() ? ECurveTableViewMode::CurveTable : ECurveTableViewMode::Grid;

	RefreshCachedCurveTable();

	return SNew(SDockTab)
		.Label( LOCTEXT("CurveTableTitle", "Curve Table") )
		.TabColorScale( GetTabColorScale() )
		[
			SNew(SBorder)
			.Padding(2)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(FMargin(8, 0))
				[
					MakeToolbar(CurveEditorPanel)
				]

				+SVerticalBox::Slot()
				[
					SNew(SSplitter)
					+SSplitter::Slot()
					.Value(.2)
					[
						SNew(SVerticalBox)
					
						+SVerticalBox::Slot()
						.Padding(0, 0, 0, 1) // adjusting padding so as to line up the rows in the cell view
						.AutoHeight()
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(2.f, 0.f, 4.f, 0.0)
							[
								SNew(SPositiveActionButton)
								.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
								.Text(LOCTEXT("Curve", "Curve"))
								.OnClicked(this, &FCurveTableEditor::OnAddCurveClicked)
								.Visibility(bTableIsReadOnly ? EVisibility::Collapsed : EVisibility::Visible)
							]

							+SHorizontalBox::Slot()	
							[
								SNew(SCurveEditorTreeTextFilter, CurveEditor)
							]
						]

						+SVerticalBox::Slot()
						[
							CurveEditorTree.ToSharedRef()
						]

					]
					+SSplitter::Slot()
					[

						SNew(SHorizontalBox)
						.Visibility(this, &FCurveTableEditor::GetTableViewControlsVisibility)

						+SHorizontalBox::Slot()
						[
							SNew(SScrollBox)
							.Orientation(Orient_Horizontal)

							+SScrollBox::Slot()
							[
								TableView.ToSharedRef()
							]
						]

						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							VerticalScrollBar
						]
					]

					+SSplitter::Slot()
					[
						SNew(SBox)
						.Visibility(this, &FCurveTableEditor::GetCurveViewControlsVisibility)
						.IsEnabled(!bTableIsReadOnly)
						[
							CurveEditorPanel
						]
					]
				]
			]
		];
}

void FCurveTableEditor::RefreshTableRows()
{
	if (TableView && CurveEditor)
	{
		TableView->RequestListRefresh();
	}
}

void FCurveTableEditor::RefreshTableRowsSelection()
{
	if(bUpdatingTableViewSelection == false && TableView && CurveEditor)
	{
		TGuardValue<bool> SelectionGuard(bUpdatingTableViewSelection, true);

		TArray<FCurveEditorTreeItemID> CurrentTreeWidgetSelection;
		TableView->GetSelectedItems(CurrentTreeWidgetSelection);
		const TMap<FCurveEditorTreeItemID, ECurveEditorTreeSelectionState>& CurrentCurveEditorTreeSelection = CurveEditor->GetTreeSelection();

		TArray<FCurveEditorTreeItemID> NewTreeWidgetSelection;
		for (const TPair<FCurveEditorTreeItemID, ECurveEditorTreeSelectionState>& CurveEditorTreeSelectionEntry : CurrentCurveEditorTreeSelection)
		{
			if (CurveEditorTreeSelectionEntry.Value != ECurveEditorTreeSelectionState::None)
			{
				NewTreeWidgetSelection.Add(CurveEditorTreeSelectionEntry.Key);
				CurrentTreeWidgetSelection.RemoveSwap(CurveEditorTreeSelectionEntry.Key);
			}
		}

		TableView->SetItemSelection(CurrentTreeWidgetSelection, false, ESelectInfo::Direct);
		TableView->SetItemSelection(NewTreeWidgetSelection, true, ESelectInfo::Direct);
	}
}

void FCurveTableEditor::OnTableViewSelectionChanged(FCurveEditorTreeItemID ItemID, ESelectInfo::Type)
{
	if (bUpdatingTableViewSelection == false && TableView && CurveEditor)
	{
		TGuardValue<bool> SelectionGuard(bUpdatingTableViewSelection, true);
		CurveEditor->GetTree()->SetDirectSelection(TableView->GetSelectedItems(), CurveEditor.Get());
	}
}

void FCurveTableEditor::RefreshCachedCurveTable()
{
	if (!CurveEditor)
	{
		return;
	}

	// This will trigger to remove any cached widgets in the TableView while we rebuild the model from the source CurveTable

	const TSet<FCurveModelID>& Pinned = CurveEditor->GetPinnedCurves();
	TSet<FName> PinnedCurves;
	for (auto PinnedCurveID : Pinned)
	{
		FCurveEditorTreeItemID TreeID = CurveEditor->GetTreeIDFromCurveID(PinnedCurveID);
		if (RowIDMap.Contains(TreeID))
		{
			PinnedCurves.Add(RowIDMap[TreeID]);
		}
	}

	TSet<FName> SelectedCurves;
	const TMap<FCurveEditorTreeItemID, ECurveEditorTreeSelectionState>& Selected = CurveEditor->GetTreeSelection();
	for (const TPair<FCurveEditorTreeItemID, ECurveEditorTreeSelectionState>& SelectionEntry: Selected)
	{
		if (SelectionEntry.Value != ECurveEditorTreeSelectionState::None)
		{
			if (RowIDMap.Contains(SelectionEntry.Key))
			{
				SelectedCurves.Add(RowIDMap[SelectionEntry.Key]);
			}
		}
	}

	// New Selection 
	TArray<FCurveEditorTreeItemID> NewSelectedItems;

	if (TableView)
	{
		TableView->SetItemsSource(&EmptyItems);
	}
	
	CurveEditor->RemoveAllTreeItems();

	ColumnNamesHeaderRow->ClearColumns();
	AvailableColumns.Empty();
	RowIDMap.Empty();

	UCurveTable* Table = GetCurveTable();
	if (!Table || Table->GetRowMap().Num() == 0)
	{
		return;
	}

	TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	const FTextBlockStyle& CellTextStyle = FAppStyle::GetWidgetStyle<FTextBlockStyle>("DataTableEditor.CellText");
	static const float CellPadding = 10.0f;

	if (Table->HasRichCurves())
	{
		InterpMode = RCIM_Cubic;
		for (const TPair<FName, FRichCurve*>& CurveRow : Table->GetRichCurveRowMap())
		{
			// Setup the CurveEdtiorTree
			const FName& CurveName = CurveRow.Key;
			FCurveEditorTreeItem* TreeItem = CurveEditor->AddTreeItem(FCurveEditorTreeItemID());
			TreeItem->SetStrongItem(MakeShared<FCurveTableEditorItem>(SharedThis(this), TreeItem->GetID(), CurveName, FCurveTableEditorHandle(Table, CurveName), AvailableColumns));
			RowIDMap.Add(TreeItem->GetID(), CurveName);

			if (SelectedCurves.Contains(CurveName))
			{
				NewSelectedItems.Add(TreeItem->GetID());
			}

			if (PinnedCurves.Contains(CurveName))
			{
				for (auto ModelID : TreeItem->GetCurves())
				{
					CurveEditor->PinCurve(ModelID);
				}
			}
		}
	}

	else
	{
		// Find unique column titles and setup columns
		TArray<float> UniqueColumns;
		for (const TPair<FName, FRealCurve*>& CurveRow : Table->GetRowMap())
		{
			FRealCurve* Curve = CurveRow.Value;
			for (auto CurveIt(Curve->GetKeyHandleIterator()); CurveIt; ++CurveIt)
			{
				UniqueColumns.AddUnique(Curve->GetKeyTime(*CurveIt));
			}
		}
		UniqueColumns.Sort();
		for (const float& ColumnTime : UniqueColumns)
		{
			const FText ColumnText = FText::AsNumber(ColumnTime);
			FCurveTableEditorColumnHeaderDataPtr CachedColumnData = MakeShareable(new FCurveTableEditorColumnHeaderData());
			CachedColumnData->ColumnId = *ColumnText.ToString();
			CachedColumnData->DisplayName = ColumnText;
			CachedColumnData->DesiredColumnWidth = FontMeasure->Measure(CachedColumnData->DisplayName, CellTextStyle.Font).X + CellPadding;
			CachedColumnData->KeyTime = ColumnTime;

			AvailableColumns.Add(CachedColumnData);

			ColumnNamesHeaderRow->AddColumn( GenerateHeaderColumnForKey(CachedColumnData) );
		}

		// Setup the CurveEditorTree 

		// Store the default Interpolation Mode
		InterpMode = RCIM_None;
		for (const TPair<FName, FSimpleCurve*>& CurveRow : Table->GetSimpleCurveRowMap())
		{
			if (InterpMode == RCIM_None) 
			{
				InterpMode = CurveRow.Value->GetKeyInterpMode();
			}

			const FName& CurveName = CurveRow.Key;
			FCurveEditorTreeItem* TreeItem = CurveEditor->AddTreeItem(FCurveEditorTreeItemID());
			TSharedPtr<FCurveTableEditorItem> NewItem = MakeShared<FCurveTableEditorItem>(SharedThis(this), TreeItem->GetID(), CurveName, FCurveTableEditorHandle(Table, CurveName), AvailableColumns);
			OnColumnsChanged.AddSP(NewItem.ToSharedRef(), &FCurveTableEditorItem::CacheKeys);
			TreeItem->SetStrongItem(NewItem);
			RowIDMap.Add(TreeItem->GetID(), CurveName);

			if (SelectedCurves.Contains(CurveName))
			{
				NewSelectedItems.Add(TreeItem->GetID());
			}

			if (PinnedCurves.Contains(CurveName))
			{
				for (auto ModelID : TreeItem->GetOrCreateCurves(CurveEditor.Get()))
				{
					CurveEditor->PinCurve(ModelID);
				}
			}
		}
	}

	if (TableView)
	{
		TableView->SetItemsSource(&CurveEditorTree->GetSourceItems());
	}

	TGuardValue<bool> SelectionGuard(bUpdatingTableViewSelection, true);
	CurveEditor->SetTreeSelection(MoveTemp(NewSelectedItems));

}

void FCurveTableEditor::OnCurveTreeViewScrolled(double InScrollOffset)
{
	const UCurveTable* CurveTable = GetCurveTable();

	// Synchronize the list views if the table has data in it
	if (TableView && ((CurveTable && CurveTable->HasRichCurves()) || AvailableColumns.Num() > 0))
	{
		TableView->SetScrollOffset(InScrollOffset);
	}
}


void FCurveTableEditor::OnTableViewScrolled(double InScrollOffset)
{
	// Synchronize the list views
	CurveEditorTree->SetScrollOffset(InScrollOffset);
}

void FCurveTableEditor::OnPostReimport(UObject* InObject, bool)
{
	const UCurveTable* Table = GetCurveTable();
	if (Table && Table == InObject)
	{
		RefreshCachedCurveTable();
	}
}

EVisibility FCurveTableEditor::GetTableViewControlsVisibility() const
{
	return ViewMode == ECurveTableViewMode::CurveTable ? EVisibility::Collapsed : EVisibility::Visible;
}

EVisibility FCurveTableEditor::GetCurveViewControlsVisibility() const
{
	return ViewMode == ECurveTableViewMode::Grid ? EVisibility::Collapsed : EVisibility::Visible;
}

void FCurveTableEditor::ToggleViewMode()
{
	ViewMode = (ViewMode == ECurveTableViewMode::CurveTable) ? ECurveTableViewMode::Grid : ECurveTableViewMode::CurveTable;
}

bool FCurveTableEditor::IsCurveViewChecked() const
{
	return (ViewMode == ECurveTableViewMode::CurveTable);
}

TSharedRef<SWidget> FCurveTableEditor::MakeToolbar(TSharedRef<SCurveEditorPanel>& InEditorPanel)
{

	FSlimHorizontalToolBarBuilder ToolBarBuilder(InEditorPanel->GetCommands(), FMultiBoxCustomization::None, InEditorPanel->GetToolbarExtender(), true);
	ToolBarBuilder.BeginSection("Asset");
	ToolBarBuilder.EndSection();
	// We just use all of the extenders as our toolbar, we don't have a need to create a separate toolbar.

	bool bHasRichCurves = GetCurveTable()->HasRichCurves();
	bool bTableIsReadOnly = IsReadOnly();

	return SNew(SHorizontalBox)

	+SHorizontalBox::Slot()
	.AutoWidth()
	.VAlign(VAlign_Center)
	[
		SNew(SBox)
		.Visibility(this, &FCurveTableEditor::GetCurveViewControlsVisibility)
		[
			ToolBarBuilder.MakeWidget()
		]
	];
}

FReply FCurveTableEditor::OnAddCurveClicked()
{
	FScopedTransaction Transaction(LOCTEXT("AddCurve", "Add Curve"));

	UCurveTable* Table = Cast<UCurveTable>(GetEditingObject());
	check(Table != nullptr);

	Table->Modify();
	if (Table->HasRichCurves())
	{
		FName NewCurveUnique = MakeUniqueCurveName(Table);
		FRichCurve& NewCurve = Table->AddRichCurve(NewCurveUnique);
		FCurveEditorTreeItem* TreeItem = CurveEditor->AddTreeItem(FCurveEditorTreeItemID());
		TreeItem->SetStrongItem(MakeShared<FCurveTableEditorItem>(SharedThis(this), TreeItem->GetID(), NewCurveUnique, FCurveTableEditorHandle(Table, NewCurveUnique), AvailableColumns));
		RowIDMap.Add(TreeItem->GetID(), NewCurveUnique);
	}
	else
	{
		FName NewCurveUnique = MakeUniqueCurveName(Table);
		FSimpleCurve& RealCurve = Table->AddSimpleCurve(NewCurveUnique);
		RealCurve.SetKeyInterpMode(InterpMode);

		// Also add a default key for each column 
		for (auto Column : AvailableColumns)
		{
			RealCurve.AddKey(Column->KeyTime, 0.0);
		}

		FCurveEditorTreeItem* TreeItem = CurveEditor->AddTreeItem(FCurveEditorTreeItemID());
		TSharedPtr<FCurveTableEditorItem> NewItem = MakeShared<FCurveTableEditorItem>(SharedThis(this), TreeItem->GetID(), NewCurveUnique, FCurveTableEditorHandle(Table, NewCurveUnique), AvailableColumns);
		OnColumnsChanged.AddSP(NewItem.ToSharedRef(), &FCurveTableEditorItem::CacheKeys);
		TreeItem->SetStrongItem(NewItem);
		RowIDMap.Add(TreeItem->GetID(), NewCurveUnique);

	}

	return FReply::Handled();
}

void FCurveTableEditor::OnAddNewKeyColumn()
{
	UCurveTable* Table = Cast<UCurveTable>(GetEditingObject());
	check(Table != nullptr);

	if (!Table->HasRichCurves())
	{
		// Compute a new keytime based on the last columns 
		float NewKeyTime = 1.0;
		if (AvailableColumns.Num() > 1)
		{
			float LastKeyTime = AvailableColumns[AvailableColumns.Num() - 1]->KeyTime;
			float PrevKeyTime = AvailableColumns[AvailableColumns.Num() - 2]->KeyTime;
			NewKeyTime = 2.*LastKeyTime - PrevKeyTime;
		}
		else if (AvailableColumns.Num() > 0)
		{
			float LastKeyTime = AvailableColumns[AvailableColumns.Num() - 1]->KeyTime;
			NewKeyTime = LastKeyTime + 1;
		}

		AddNewKeyColumn(NewKeyTime);
	}
}

void FCurveTableEditor::AddNewKeyColumn(float NewKeyTime)
{
	UCurveTable* Table = Cast<UCurveTable>(GetEditingObject());
	check(Table != nullptr);

	if (!Table->HasRichCurves())
	{
		FScopedTransaction Transaction(LOCTEXT("AddKeyColumn", "AddKeyColumn"));
		Table->Modify();	

		// Make sure we don't already have a key at this time
		
		// 1. Add new keys to every curve
		for (const TPair<FName, FRealCurve*>& CurveRow : Table->GetRowMap())
		{
			FRealCurve* Curve = CurveRow.Value;
			Curve->UpdateOrAddKey(NewKeyTime, Curve->Eval(NewKeyTime));
		}

		// 2. Add Column to our Table
		FCurveTableEditorColumnHeaderDataPtr ColumnData = MakeShareable(new FCurveTableEditorColumnHeaderData());
		const FText ColumnText = FText::AsNumber(NewKeyTime);
		ColumnData->ColumnId = *ColumnText.ToString();
		ColumnData->DisplayName = ColumnText;
		ColumnData->KeyTime = NewKeyTime;

		TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		const FTextBlockStyle& CellTextStyle = FAppStyle::GetWidgetStyle<FTextBlockStyle>("DataTableEditor.CellText");
		ColumnData->DesiredColumnWidth = FontMeasure->Measure(ColumnData->DisplayName, CellTextStyle.Font).X + 10.f;

		AvailableColumns.Add(ColumnData);

		// 3. Let the CurveTreeItems know they need to recache
		OnColumnsChanged.Broadcast();

		ColumnNamesHeaderRow->AddColumn( GenerateHeaderColumnForKey(ColumnData) );
	}
}

void FCurveTableEditor::OnRequestCurveRename(FCurveEditorTreeItemID TreeItemId)
{
	const FCurveEditorTreeItem* TreeItem = CurveEditor->FindTreeItem(TreeItemId);
	if (TreeItem != nullptr)
	{
		TSharedPtr<ICurveEditorTreeItem> CurveEditorTreeItem = TreeItem->GetItem();
		if (CurveEditorTreeItem.IsValid())
		{
			TSharedPtr<FCurveTableEditorItem> CurveTableEditorItem = StaticCastSharedPtr<FCurveTableEditorItem>(CurveEditorTreeItem);
			CurveTableEditorItem->EnterRenameMode();
		}
	}
}

void FCurveTableEditor::HandleCurveRename(FCurveEditorTreeItemID& TreeID, FName& CurrentCurve, FName& NewCurveName)
{
	// Update the underlying Curve Data Asset itself 
	UCurveTable* Table = Cast<UCurveTable>(GetEditingObject());
	check(Table != nullptr);

	FScopedTransaction Transaction(LOCTEXT("RenameCurve", "Rename Curve"));
	Table->SetFlags(RF_Transactional);
	Table->Modify();
	Table->RenameRow(CurrentCurve, NewCurveName);

	FPropertyChangedEvent PropertyChangeStruct(nullptr, EPropertyChangeType::ValueSet);
	Table->PostEditChangeProperty(PropertyChangeStruct);

	// Update our internal map of TreeIDs to FNames
	RowIDMap[TreeID] = NewCurveName;

}

void FCurveTableEditor::OnRenameCurve()
{
	const TMap<FCurveEditorTreeItemID, ECurveEditorTreeSelectionState>& SelectedRows = CurveEditor->GetTreeSelection();
	if (SelectedRows.Num() == 1)
	{
		for (auto Item : SelectedRows)
		{
			OnRequestCurveRename(Item.Key);
		}		
	}
}

void FCurveTableEditor::OnDeleteCurves()
{
	UCurveTable* Table = Cast<UCurveTable>(GetEditingObject());
	check(Table != nullptr);

	const TMap<FCurveEditorTreeItemID, ECurveEditorTreeSelectionState> SelectedRows = CurveEditor->GetTreeSelection();

	if (SelectedRows.Num() >= 1)
	{
		FScopedTransaction Transaction(LOCTEXT("DeleteCurveRow", "Delete Curve Rows"));
		Table->SetFlags(RF_Transactional);
		Table->Modify();

		for (auto Item : SelectedRows)
		{
			CurveEditor->RemoveTreeItem(Item.Key);

			FName& CurveName = RowIDMap[Item.Key];

			Table->DeleteRow(CurveName);

			RowIDMap.Remove(Item.Key);
		}

		FPropertyChangedEvent PropertyChangeStruct(nullptr, EPropertyChangeType::ValueSet);
		Table->PostEditChangeProperty(PropertyChangeStruct);
	}
}

TSharedPtr<SWidget> FCurveTableEditor::OnOpenCurveMenu()
{
	int32 SelectedRowCount = CurveEditor->GetTreeSelection().Num();
	if (SelectedRowCount > 0 && !IsReadOnly())
	{
		FMenuBuilder MenuBuilder(true /*auto close*/, ToolkitCommands);
		MenuBuilder.BeginSection("Edit");
		if (SelectedRowCount == 1)
		{
			MenuBuilder.AddMenuEntry(
				FCurveTableEditorCommands::Get().RenameSelectedCurve,
				NAME_None,
				TAttribute<FText>(),
				TAttribute<FText>(),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Edit")
			);
		}
		MenuBuilder.AddMenuEntry(
			FCurveTableEditorCommands::Get().DeleteSelectedCurves,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Delete")
		);
		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	}

	return SNullWidget::NullWidget;
}

void FCurveTableEditor::OnDeleteKeyColumn(float KeyTime)
{
	UCurveTable* Table = Cast<UCurveTable>(GetEditingObject());
	check(Table != nullptr);

	if (!Table->HasRichCurves())
	{
		// First find the column data associated with the original keytime
		int FoundIndex = -1;
		for (int i = 0; i < AvailableColumns.Num(); i++)
		{
			if (FMath::IsNearlyEqual(KeyTime, AvailableColumns[i]->KeyTime, KINDA_SMALL_NUMBER))
			{
				FoundIndex = i;
				break;
			}
		}
		if (FoundIndex < 0)
		{
			return;
		}

		FCurveTableEditorColumnHeaderDataPtr ColumnData = AvailableColumns[FoundIndex];
		if (ColumnData.IsValid())
		{

			// Remove the column from the ui 
			AvailableColumns.RemoveAt(FoundIndex);
			ColumnNamesHeaderRow->RemoveColumn(ColumnData->ColumnId);

			// Remove the keys from all curve rows is the data table
			FScopedTransaction Transaction(LOCTEXT("DeleteKeyColumn", "Delete Key Column"));
			Table->Modify();	

			for (const TPair<FName, FRealCurve*>& CurveRow : Table->GetRowMap())
			{
				FRealCurve* Curve = CurveRow.Value;
				FKeyHandle KeyHandle = Curve->FindKey(KeyTime);
				if (KeyHandle != FKeyHandle::Invalid())
				{
					Curve->DeleteKey(KeyHandle);
				}
			}

			FPropertyChangedEvent PropertyChangeStruct(nullptr, EPropertyChangeType::ValueSet);
			Table->PostEditChangeProperty(PropertyChangeStruct);

			// Let the CurveTreeItems (row ui) know they need to recache
			OnColumnsChanged.Broadcast();
		}

	}
}

bool FCurveTableEditor::VerifyValidRetime(const FText& InText, FText& OutErrorMessage, float OriginalTime)
{
	if (!InText.IsNumeric())
	{
		OutErrorMessage = LOCTEXT("KeysMustBeNumeric", "Key Times must be numeric.");
		return false;
	}

	float NewTime = 0.0f;
	LexFromString(NewTime, *InText.ToString());

	// do we already have a column with this time? 
	for (auto Col : AvailableColumns)
	{
		if (FMath::IsNearlyEqual(NewTime, Col->KeyTime, KINDA_SMALL_NUMBER))
		{
			OutErrorMessage = LOCTEXT("KeyAlreadyExists", "Key times must be unique!");
			return false;
		}
	}
	return true;
}

void FCurveTableEditor::HandleRetimeCommitted(const FText& InText, ETextCommit::Type CommitInfo, float OriginalKeyTime)
{

	// First find the column data associated with the original keytime
	int FoundIndex = -1;
	for (int i = 0; i < AvailableColumns.Num(); i++)
	{
		if (FMath::IsNearlyEqual(OriginalKeyTime, AvailableColumns[i]->KeyTime, KINDA_SMALL_NUMBER))
		{
			FoundIndex = i;
			break;
		}
	}
	if (FoundIndex < 0)
	{
		return;
	}

	FCurveTableEditorColumnHeaderDataPtr CachedColumnData = AvailableColumns[FoundIndex];
	if (CachedColumnData.IsValid())
	{
		// 1. Remove the UI associated with this column (ColumnData and the SHeaderRow::FColumn)
		AvailableColumns.RemoveAt(FoundIndex);
		ColumnNamesHeaderRow->RemoveColumn(CachedColumnData->ColumnId);

		float NewTime = 0.0f;
		LexFromString(NewTime, *InText.ToString());
	
		// 2. Adjust the key times for each of the curve table rows
		UCurveTable* Table = Cast<UCurveTable>(GetEditingObject());
		check(Table != nullptr);

		FScopedTransaction Transaction(LOCTEXT("RetimeKeyColumn", "Retime Key Column"));
		Table->Modify();	

		for (const TPair<FName, FRealCurve*>& CurveRow : Table->GetRowMap())
		{
			FRealCurve* Curve = CurveRow.Value;
			FKeyHandle KeyHandle = Curve->FindKey(OriginalKeyTime);
			if (KeyHandle != FKeyHandle::Invalid())
			{
				Curve->SetKeyTime(KeyHandle, NewTime);
			}
		}

		FPropertyChangedEvent PropertyChangeStruct(nullptr, EPropertyChangeType::ValueSet);
		Table->PostEditChangeProperty(PropertyChangeStruct);


		// 3. Update the ColumnData and re-insert the ColumnData and SHeaderRow::FColumn into the 
		// correct places in order of the key times
		int NewIndex = 0;
		while (NewIndex < AvailableColumns.Num() && NewTime > AvailableColumns[NewIndex]->KeyTime )
		{
			NewIndex++;
		}

		const FText ColumnText = FText::AsNumber(NewTime);
		CachedColumnData->ColumnId = *ColumnText.ToString();
		CachedColumnData->DisplayName = ColumnText;
		CachedColumnData->KeyTime = NewTime;

		AvailableColumns.Insert(CachedColumnData, NewIndex);

		// Let the CurveTreeItems know they need to recache
		// note we do this before adding the column to the header so the rows already have their 
		// data in place and are prepared to draw
		OnColumnsChanged.Broadcast();

		ColumnNamesHeaderRow->InsertColumn( GenerateHeaderColumnForKey(CachedColumnData), NewIndex)	;
	}
}

SHeaderRow::FColumn::FArguments FCurveTableEditor::GenerateHeaderColumnForKey(FCurveTableEditorColumnHeaderDataPtr ColumnData)
{
	TSharedRef<SInlineEditableTextBlock> KeyTimeWidget = SNew(SInlineEditableTextBlock)
	.Text(ColumnData->DisplayName)
	.Justification(ETextJustify::Center)
	.ColorAndOpacity(FSlateColor::UseForeground())
	.OnTextCommitted(this, &FCurveTableEditor::HandleRetimeCommitted, ColumnData->KeyTime)
	.OnVerifyTextChanged(this, &FCurveTableEditor::VerifyValidRetime, ColumnData->KeyTime);

	// Create the Column Header's R-Click Menu
	FMenuBuilder MenuBuilder(true /*Auto close*/, ToolkitCommands);
	MenuBuilder.BeginSection("Edit");
	MenuBuilder.AddMenuEntry(
		FText::Format(LOCTEXT("RetimeKeysColumn", "Retime Keys at  {0}"), FText::AsNumber(ColumnData->KeyTime)),
		FText::Format(LOCTEXT("RetimeKeysColumn_Tooltip", "Retimes this column and all keys at  {0}"), FText::AsNumber(ColumnData->KeyTime)),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Edit"),
		FUIAction(FExecuteAction::CreateSP(KeyTimeWidget, &SInlineEditableTextBlock::EnterEditingMode))
	);
	MenuBuilder.AddMenuEntry(
		FText::Format(LOCTEXT("DeleteKeysColumn", "Delete Keys at  {0}"), FText::AsNumber(ColumnData->KeyTime)),
		FText::Format(LOCTEXT("DeleteKeysColumn_Tooltip", "Deletes this column and all keys at  {0}"), FText::AsNumber(ColumnData->KeyTime)),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Delete"),
		FUIAction(FExecuteAction::CreateSP(this, &FCurveTableEditor::OnDeleteKeyColumn, ColumnData->KeyTime))
	);
	MenuBuilder.EndSection();

	return SHeaderRow::Column(ColumnData->ColumnId)
		.DefaultLabel(ColumnData->DisplayName)
		.FixedWidth(ColumnData->DesiredColumnWidth + 40)
		.HAlignHeader(HAlign_Fill)
		.MenuContent()
		[
			MenuBuilder.MakeWidget()
		]
		.HeaderContent()
		[
			SNew(SBox)
			.HeightOverride(22.f)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				KeyTimeWidget
			]
		];
}

#undef LOCTEXT_NAMESPACE