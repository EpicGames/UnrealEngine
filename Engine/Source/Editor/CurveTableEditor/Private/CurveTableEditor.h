// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "CurveEditorTypes.h"
#include "CurveTableEditorUtils.h"
#include "Curves/RealCurve.h"
#include "Delegates/Delegate.h"
#include "EditorUndoClient.h"
#include "Framework/Docking/TabManager.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "ICurveTableEditor.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "Toolkits/IToolkit.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/SListView.h"

class FCurveEditor;
class FExtender;
class SCurveEditorPanel;
class SDockTab;
class SWidget;
class UCurveTable;
class UObject;


/** The manner in which curve tables are displayed */
enum class ECurveTableViewMode : int32
{
	/** Displays values in a spreadsheet-like table */
	Grid,

	/** Displays values as curves */
	CurveTable,
};

struct FCurveTableEditorColumnHeaderData;

typedef TSharedPtr<FCurveTableEditorColumnHeaderData> FCurveTableEditorColumnHeaderDataPtr;

/** Viewer/Editor for a CurveTable */
class FCurveTableEditor :
	public ICurveTableEditor
	, public FCurveTableEditorUtils::INotifyOnCurveTableChanged
	, public FEditorUndoClient
{

public:
	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

	/**
	 * Edits the specified table
	 *
	 * @param	Mode					Asset editing mode for this editor (standalone or world-centric)
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	Table					The table to edit
	 */
	virtual void InitCurveTableEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UCurveTable* Table );

	/** Destructor */
	virtual ~FCurveTableEditor();

	/** IToolkit interface */
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;

	// INotifyOnDataTableChanged
	virtual void PreChange(const UCurveTable* Changed, FCurveTableEditorUtils::ECurveTableChangeInfo Info) override;
	virtual void PostChange(const UCurveTable* Changed, FCurveTableEditorUtils::ECurveTableChangeInfo Info) override;
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;

	/** Get the curve table being edited */
	UCurveTable* GetCurveTable() const;

	/** Determine if the table being viewed, can be edited or is read-only */
	bool IsReadOnly() const;

	/** Get the curve editor used for the Curve View*/
	TSharedPtr<FCurveEditor> GetCurveEditor() const { return CurveEditor; }

	void HandlePostChange();

	/**	Spawns the tab with the curve table inside */
	TSharedRef<SDockTab> SpawnTab_CurveTable( const FSpawnTabArgs& Args );

	/** Get the mode that we are displaying data in */
	ECurveTableViewMode GetViewMode() const { return ViewMode; }

	/** Rename a specific curve */
	void HandleCurveRename(FCurveEditorTreeItemID& TreeID, FName& OldCurveName, FName& NewCurveName);

	/** Pop open context Menu */
	TSharedPtr<SWidget> OnOpenCurveMenu();

	/** Callback for R-Click Menu Delete Curves */
	void OnDeleteCurves();

	/** Callback for R-Click Menu Rename Selected Curve */
	void OnRenameCurve();

	/** Callback for R-Click Menu Delete Key Column */
	void OnDeleteKeyColumn(float KeyTime);

	/** Ensure that whatever time we are reetiming to is not redundant */
	bool VerifyValidRetime(const FText& InText, FText& OutErrorMessage, float OriginalTime);

	/** Process the retime from the header column text entry */
	void HandleRetimeCommitted(const FText& InText, ETextCommit::Type CommitInfo, float OriginalKeyTime);

protected:

	/** Handles setting up slate for the curve table editor */
	virtual TSharedRef< FTabManager::FLayout > InitCurveTableLayout();

	/** Add extra menu items */
	void ExtendMenu();

	/** Add extra menu items */
	void ExtendToolbar();

	/** Bind commands to delegates */
	void BindCommands();

	/** Update the cached state of this curve table, and then reflect that new state in the UI */
	void RefreshCachedCurveTable();

	/** Make the toolbar */
	TSharedRef<SWidget> MakeToolbar(TSharedRef<SCurveEditorPanel>& CurveEditorPanel);

	/** Called when the CurveEditorTree view is scrolled - used to keep the two list views in sync */
	void OnCurveTreeViewScrolled(double InScrollOffset);

	/** Called when the Table View is scrolled - used to keep the two list views in sync */
	void OnTableViewScrolled(double InScrollOffset);

	/** Called when someone selected a row directly in the TableView - used to keep selection in sync between CurveTree  and TableView */
	void OnTableViewSelectionChanged(FCurveEditorTreeItemID ItemID, ESelectInfo::Type);

	/** Called when an asset has finished being imported */
	void OnPostReimport(UObject* InObject, bool);

	/** Control control visibility based on view mode */
	EVisibility GetTableViewControlsVisibility() const;

	/** Control control visibility based on view mode */
	EVisibility GetCurveViewControlsVisibility() const;

	/** Add New Curve Callback */
	FReply OnAddCurveClicked();

	/** Callback For SimpleCurves, add a new Key/Column */
	void OnAddNewKeyColumn();

	/* Adds new key for all (Simple) curves in the table at given time */
	void AddNewKeyColumn(float NewKeyTime);

	/** Toggle between curve & grid view */
	void ToggleViewMode();

	/** Get whether the curve view checkbox should be toggled on */
	bool IsCurveViewChecked() const;

	/** Invoke UI for Renaming a Curve */
	void OnRequestCurveRename(FCurveEditorTreeItemID TreeItemId);

	virtual bool ShouldCreateDefaultStandaloneMenu() const { return true; }
	virtual bool ShouldCreateDefaultToolbar() const { return true ; }

	SHeaderRow::FColumn::FArguments GenerateHeaderColumnForKey(FCurveTableEditorColumnHeaderDataPtr ColumnData);

	/** Array of the columns that are available for editing */
	TArray<FCurveTableEditorColumnHeaderDataPtr> AvailableColumns;

	/** Header row containing entries for each column in AvailableColumns */
	TSharedPtr<SHeaderRow> ColumnNamesHeaderRow;

	/** List view responsible for showing the rows from AvailableColumns */
	TSharedPtr<SListView<FCurveEditorTreeItemID>> TableView;

	/** Menu extender */
	TSharedPtr<FExtender> MenuExtender;

	/** Menu extender */
	TSharedPtr<FExtender> ToolbarExtender;

	/**	The tab id for the curve table tab */
	static const FName CurveTableTabId;

	/** The manner in which curve tables are displayed */
	ECurveTableViewMode ViewMode;

	/** The Curve Editor */
	TSharedPtr<FCurveEditor> CurveEditor;

	/* The Data Model that holds the source items for Views (TreeView, TableView) */
	TSharedPtr<class SCurveEditorTree> CurveEditorTree;

	bool bUpdatingTableViewSelection;

	/* Sync Filtered from the CurveEditorTree model to the TableView*/
	void RefreshTableRows();

	/* Sync selected rows from the CurveEditorTree model to the TableView */
	void RefreshTableRowsSelection();

	/** A delegate to let item rows know when the number of columns have changed */
	FSimpleMulticastDelegate OnColumnsChanged;

	/** An empty source list used to initialize or when rebuilding the TableView */
	TArray<FCurveEditorTreeItemID> EmptyItems;

	/** Associates the CurveEditor Ids with the FName Ids from the Curve Table */
	TMap< FCurveEditorTreeItemID, FName > RowIDMap;

	/** The default interpolation type of the curves within the table */
	ERichCurveInterpMode InterpMode;

};
