// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveTableEditorCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"

#define LOCTEXT_NAMESPACE "CurveTableEditorCommands"

void FCurveTableEditorCommands::RegisterCommands()
{
	UI_COMMAND(CurveViewToggle, "Curve View", "Changes the view of the curve table from grid to curve view.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(AppendKeyColumn, "Append Key Column", "Append a new column to the curve table.\nEvery Curve or Table Row will have a new key appended.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DeleteSelectedCurves, "Delete Curves", "Deleted the selected curve rows.", EUserInterfaceActionType::None, FInputChord());
	UI_COMMAND(RenameSelectedCurve, "Rename Curve", "Rename the selected curve row.", EUserInterfaceActionType::None, FInputChord());
}

#undef LOCTEXT_NAMESPACE
