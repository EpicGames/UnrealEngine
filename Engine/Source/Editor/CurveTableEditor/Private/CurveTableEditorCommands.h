// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;

class FCurveTableEditorCommands : public TCommands<FCurveTableEditorCommands>
{
public:
	FCurveTableEditorCommands()
		: TCommands<FCurveTableEditorCommands>(TEXT("CurveTableEditor"), NSLOCTEXT("Contexts", "CurveTableEditor", "Curve Table Editor"), NAME_None, FAppStyle::GetAppStyleSetName())
	{
	}

	virtual void RegisterCommands() override;

public:

	// Command to allow users to toggle the view mode
	TSharedPtr<FUICommandInfo> CurveViewToggle;
	TSharedPtr<FUICommandInfo> AppendKeyColumn;
	TSharedPtr<FUICommandInfo> RenameSelectedCurve;
	TSharedPtr<FUICommandInfo> DeleteSelectedCurves;
};
