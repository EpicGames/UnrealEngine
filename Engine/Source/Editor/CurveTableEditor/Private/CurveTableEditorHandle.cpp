// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveTableEditorHandle.h"

#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "UObject/ObjectMacros.h"

class UObject;
struct FRealCurve;

FRealCurve* FCurveTableEditorHandle::GetCurve() const
{
	if (CurveTable != nullptr && RowName != NAME_None)
	{
		return CurveTable.Get()->FindCurve(RowName, TEXT("CurveTableEditorHandle::GetCurve"));
	}
	return nullptr;
}

FRichCurve* FCurveTableEditorHandle::GetRichCurve() const
{
	if (CurveTable != nullptr && RowName != NAME_None)
	{
		return CurveTable.Get()->FindRichCurve(RowName, TEXT("CurveTableEditorHandle::GetCurve"));
	}
	return nullptr;
}

bool FCurveTableEditorHandle::HasRichCurves() const
{
	if (CurveTable != nullptr)
	{
		return CurveTable.Get()->HasRichCurves();
	}
	return false;
}

TArray<FRichCurveEditInfoConst> FCurveTableEditorHandle::GetCurves() const
{
	TArray<FRichCurveEditInfoConst> Curves;

	const FRealCurve* Curve = GetCurve();
	if (Curve)
	{
		Curves.Add(FRichCurveEditInfoConst(Curve, RowName));
	}

	return Curves;
}

void FCurveTableEditorHandle::GetCurves(TAdderReserverRef<FRichCurveEditInfoConst> Curves) const
{
	if (const FRealCurve* Curve = GetCurve())
	{
		Curves.Add(FRichCurveEditInfoConst(Curve, RowName));
	}
}

TArray<FRichCurveEditInfo> FCurveTableEditorHandle::GetCurves()
{
	TArray<FRichCurveEditInfo> Curves;

	FRealCurve* Curve = GetCurve();
	if (Curve)
	{
		Curves.Add(FRichCurveEditInfo(Curve, RowName));
	}

	return Curves;
}

TArray<const UObject*> FCurveTableEditorHandle::GetOwners() const
{ 
	TArray<const UObject*> Owners;
	if (CurveTable != nullptr)
	{
		Owners.Add(CurveTable.Get());
	}

	return Owners;
}

void FCurveTableEditorHandle::ModifyOwner()
{
	if (CurveTable != nullptr && RowName != NAME_None)
	{
		CurveTable->Modify();
	}

}

void FCurveTableEditorHandle::MakeTransactional()
{
	if (CurveTable != nullptr)
	{
		CurveTable->SetFlags(CurveTable->GetFlags() | RF_Transactional);
	}
}

void FCurveTableEditorHandle::OnCurveChanged(const TArray<FRichCurveEditInfo>& ChangedCurveEditInfos)
{

}

bool FCurveTableEditorHandle::IsValidCurve(FRichCurveEditInfo CurveInfo)
{
	return CurveInfo.CurveToEdit == GetCurve();
}
