// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Curves/CurveOwnerInterface.h"
#include "Curves/RichCurve.h"
#include "Engine/CurveTable.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UObject;
struct FRealCurve;

/**
 * Handle to a particular row in a table, used for editing individual curves
 */
struct FCurveTableEditorHandle : public FCurveOwnerInterface
{
	FCurveTableEditorHandle()
		: CurveTable(nullptr)
		, RowName(NAME_None)
	{ }

	FCurveTableEditorHandle(UCurveTable* InCurveTable, FName InRowName)
		: CurveTable(InCurveTable)
		, RowName(InRowName)
	{ }

	/** Pointer to table we want a row from */
	TWeakObjectPtr<UCurveTable> CurveTable;

	/** Name of row in the table that we want */
	FName RowName;

	//~ Begin FCurveOwnerInterface Interface.
	UE_DEPRECATED(5.6, "Use version taking a TAdderReserverRef")
	virtual TArray<FRichCurveEditInfoConst> GetCurves() const override;
	virtual void GetCurves(TAdderReserverRef<FRichCurveEditInfoConst> Curves) const override;
	virtual TArray<FRichCurveEditInfo> GetCurves() override;
	virtual void ModifyOwner() override;
	virtual void MakeTransactional() override;
	virtual void OnCurveChanged(const TArray<FRichCurveEditInfo>& ChangedCurveEditInfos) override;
	virtual bool IsValidCurve(FRichCurveEditInfo CurveInfo) override;
	virtual TArray<const UObject*> GetOwners() const override;
	

	//~ End FCurveOwnerInterface Interface.

	/** Returns true if the curve is valid */
	bool IsValid() const
	{
		return (GetCurve() != nullptr);
	}

	/** Returns true if this handle is specifically pointing to nothing */
	bool IsNull() const
	{
		return CurveTable == nullptr && RowName == NAME_None;
	}

	/** Get the curve straight from the row handle */
	FRealCurve* GetCurve() const;
	FRichCurve* GetRichCurve() const;

	/** Returns true if the owning table uses RichCurves instead of Real Curves */
	bool HasRichCurves() const;
};
