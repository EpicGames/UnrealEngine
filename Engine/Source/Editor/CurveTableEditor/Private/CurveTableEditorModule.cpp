// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveTableEditorModule.h"

#include "CompositeCurveTableEditor.h"
#include "CurveTableEditor.h"
#include "Engine/CompositeCurveTable.h"
#include "Engine/CurveTable.h"
#include "HAL/Platform.h"
#include "Modules/ModuleManager.h"
#include "Templates/Casts.h"
#include "UObject/NameTypes.h"

class IToolkitHost;

IMPLEMENT_MODULE( FCurveTableEditorModule, CurveTableEditor );


const FName FCurveTableEditorModule::CurveTableEditorAppIdentifier( TEXT( "CurveTableEditorApp" ) );

void FCurveTableEditorModule::StartupModule()
{
	MenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
}


void FCurveTableEditorModule::ShutdownModule()
{
	MenuExtensibilityManager.Reset();
}


TSharedRef<ICurveTableEditor> FCurveTableEditorModule::CreateCurveTableEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UCurveTable* Table )
{
	if (Cast<UCompositeCurveTable>(Table) != nullptr)
	{
		return CreateCompositeCurveTableEditor(Mode, InitToolkitHost, Table);
	}

	return CreateStandardCurveTableEditor(Mode, InitToolkitHost, Table);
}

TSharedRef<ICurveTableEditor> FCurveTableEditorModule::CreateStandardCurveTableEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UCurveTable* Table)
{
	TSharedRef< FCurveTableEditor > NewCurveTableEditor(new FCurveTableEditor());
	NewCurveTableEditor->InitCurveTableEditor(Mode, InitToolkitHost, Table);
	return NewCurveTableEditor;
}

TSharedRef<ICurveTableEditor> FCurveTableEditorModule::CreateCompositeCurveTableEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UCurveTable* Table)
{
	TSharedRef< FCompositeCurveTableEditor > NewCurveTableEditor(new FCompositeCurveTableEditor());
	NewCurveTableEditor->InitCurveTableEditor(Mode, InitToolkitHost, Table);
	return NewCurveTableEditor;
}

