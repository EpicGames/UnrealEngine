// Copyright Epic Games, Inc. All Rights Reserved.

#include "RealCurveModel.h"

#include "CurveDataAbstraction.h"
#include "CurveDrawInfo.h"
#include "CurveEditorScreenSpace.h"
#include "Curves/KeyHandle.h"
#include "Curves/RealCurve.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "Math/Vector2D.h"
#include "Misc/Optional.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtr.h"

class FCurveEditor;

FRealCurveModel::FRealCurveModel(FCurveTableEditorHandle InRealCurveHandle, UObject* InOwner)
	: WeakOwner(InOwner)
	, RealCurveHandle(MoveTemp(InRealCurveHandle))
{}

const void* FRealCurveModel::GetCurve() const
{
	return ResolveCurve();
}

void FRealCurveModel::Modify()
{
	if (UObject* Owner = WeakOwner.Get())
	{
		if(IsValid())
		{
			Owner->SetFlags(RF_Transactional);
			Owner->Modify();
		}
	}	
}

void FRealCurveModel::DrawCurve(const FCurveEditor& CurveEditor, const FCurveEditorScreenSpace& ScreenSpace, TArray<TTuple<double, double>>& InterpolatingPoints) const
{
	if (FRealCurve* RealCurve = ResolveCurve())
	{

		FKeyHandle LastKeyHandle = RealCurve->GetFirstKeyHandle();
		double LastKeyTime =  double(ScreenSpace.GetInputMin());
		double LastKeyValue =  double(RealCurve->GetKeyValue(LastKeyHandle));
		double LastInterpMode = RealCurve->GetKeyInterpMode(LastKeyHandle);

		if (LastInterpMode == (double)RCIM_Constant)
		{
			InterpolatingPoints.Add(MakeTuple( LastKeyTime, LastKeyValue));
		}

		for (auto It = RealCurve->GetKeyHandleIterator(); It; ++It)
		{
			auto KeyPair = RealCurve->GetKeyTimeValuePair(*It);

			// if  constant , add another point to mark the end of the previous' key reign
			if (LastInterpMode == (double)RCIM_Constant)
			{
				InterpolatingPoints.Add(MakeTuple( double(KeyPair.Key), LastKeyValue));
			}

			InterpolatingPoints.Add(MakeTuple( double(KeyPair.Key), double(KeyPair.Value)));

			LastKeyHandle = RealCurve->GetNextKey(LastKeyHandle);
			LastKeyValue = KeyPair.Value;
			LastInterpMode = RealCurve->GetKeyInterpMode(LastKeyHandle);
		}

		if (LastInterpMode == (double)RCIM_Constant)
		{
			InterpolatingPoints.Add(MakeTuple( double(ScreenSpace.GetInputMax()), LastKeyValue));
		}
	}
}

void FRealCurveModel::GetKeys(double MinTime, double MaxTime, double MinValue, double MaxValue, TArray<FKeyHandle>& OutKeyHandles) const
{
	if (FRealCurve* RealCurve = ResolveCurve())
	{
		for (auto It = RealCurve->GetKeyHandleIterator(); It; ++It)
		{
			auto KeyPair = RealCurve->GetKeyTimeValuePair(*It);
			if (KeyPair.Key >= MinTime && KeyPair.Key <= MaxTime && KeyPair.Value >= MinValue && KeyPair.Value <= MaxValue)
			{
				OutKeyHandles.Add(*It);
			}
		}
	}
}

void FRealCurveModel::AddKeys(TArrayView<const FKeyPosition> InKeyPositions, TArrayView<const FKeyAttributes> InAttributes, TArrayView<TOptional<FKeyHandle>>* OutKeyHandles)
{
	// Left empty for now - The curve editor cannot directly add/remove keys to RealCurves
}

void FRealCurveModel::RemoveKeys(TArrayView<const FKeyHandle> InKeys, double InCurrentTime)
{
	// Left empty for now - The curve editor cannot directly add/remove keys to RealCurves
}

void FRealCurveModel::GetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyPosition> OutKeyPositions) const
{
	if (FRealCurve* RealCurve = ResolveCurve())
	{
		for (int32 Index = 0; Index < InKeys.Num(); ++Index)
		{
			if (RealCurve->IsKeyHandleValid(InKeys[Index]))
			{
				auto KeyPair = RealCurve->GetKeyTimeValuePair(InKeys[Index]);
				OutKeyPositions[Index].InputValue = KeyPair.Key;
				OutKeyPositions[Index].OutputValue = KeyPair.Value;
			}
		}
	}
}

void FRealCurveModel::SetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyPosition> InKeyPositions, EPropertyChangeType::Type ChangeType)
{
	if (UObject* Owner = WeakOwner.Get())
	{
		if (FRealCurve* RealCurve = ResolveCurve())
		{
			Owner->Modify();

			for (int32 Index = 0; Index < InKeys.Num(); ++Index)
			{
				if (RealCurve->IsKeyHandleValid(InKeys[Index]))
				{
					// For now, you can only change RealCurves' position, not time 
					RealCurve->SetKeyValue(InKeys[Index], InKeyPositions[Index].OutputValue);
				}	
			}
			FPropertyChangedEvent PropertyChangeStruct(nullptr, EPropertyChangeType::ValueSet);
			Owner->PostEditChangeProperty(PropertyChangeStruct);

			CurveModifiedDelegate.Broadcast();
		}	
	}
}

void FRealCurveModel::GetKeyDrawInfo(ECurvePointType PointType, const FKeyHandle InKeyHandle, FKeyDrawInfo& OutDrawInfo) const
{
	if (PointType == ECurvePointType::ArriveTangent || PointType == ECurvePointType::LeaveTangent)
	{
		OutDrawInfo.Brush = FAppStyle::Get().GetBrush("GenericCurveEditor.TangentHandle");
		OutDrawInfo.ScreenSize = FVector2D(9, 9);
	}
	else if (FRealCurve* RealCurve = ResolveCurve())
	{
		// All keys are the same size by default
		OutDrawInfo.ScreenSize = FVector2D(11, 11);

		ERichCurveInterpMode KeyType = (IsValid() && RealCurve->IsKeyHandleValid(InKeyHandle)) ? RealCurve->GetKeyInterpMode(InKeyHandle) : RCIM_None;
		switch (KeyType)
		{
		case ERichCurveInterpMode::RCIM_Constant:
			OutDrawInfo.Brush = FAppStyle::Get().GetBrush("GenericCurveEditor.ConstantKey");
			break;
		case ERichCurveInterpMode::RCIM_Linear:
			OutDrawInfo.Brush = FAppStyle::Get().GetBrush("GenericCurveEditor.LinearKey");
			break;
		case ERichCurveInterpMode::RCIM_Cubic:
			OutDrawInfo.Brush = FAppStyle::Get().GetBrush("GenericCurveEditor.CubicKey");
			break;
		default:
			OutDrawInfo.Brush = FAppStyle::Get().GetBrush("GenericCurveEditor.Key");
			break;
		}
	}
}

void FRealCurveModel::GetKeyAttributes(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyAttributes> OutAttributes) const
{
	if(FRealCurve* RealCurve = ResolveCurve())
	{
		for (int32 Index = 0; Index < InKeys.Num(); ++Index)
		{
			if (RealCurve->IsKeyHandleValid(InKeys[Index]))
			{
				FKeyAttributes&      Attributes = OutAttributes[Index];
				Attributes.SetInterpMode( RealCurve->GetKeyInterpMode(InKeys[Index]) );
			}
		}
	}
}

void FRealCurveModel::GetTimeRange(double& MinTime, double& MaxTime) const
{
	if (FRealCurve* RealCurve = ResolveCurve())
	{
		float MinT, MaxT; 
		RealCurve->GetTimeRange(MinT, MaxT);
		MinTime = MinT;
		MaxTime = MaxT;
	}
}

void FRealCurveModel::GetValueRange(double& MinValue, double& MaxValue) const
{
	if (FRealCurve* RealCurve = ResolveCurve())
	{
		float MinV, MaxV;
		RealCurve->GetValueRange(MinV, MaxV);
		MinValue = MinV;
		MaxValue = MaxV;
	}
}

int32 FRealCurveModel::GetNumKeys() const
{
	if (FRealCurve* RealCurve = ResolveCurve())
	{
		return RealCurve->GetNumKeys();
	}
	return 0;
}

void FRealCurveModel::GetNeighboringKeys(const FKeyHandle InKeyHandle, TOptional<FKeyHandle>& OutPreviousKeyHandle, TOptional<FKeyHandle>& OutNextKeyHandle) const
{
	if (FRealCurve* RealCurve = ResolveCurve())
	{
		OutPreviousKeyHandle = RealCurve->GetPreviousKey(InKeyHandle);
		OutNextKeyHandle = RealCurve->GetNextKey(InKeyHandle);
	}	
}

bool FRealCurveModel::Evaluate(double ProspectiveTime, double& OutValue) const
{
	if (FRealCurve* RealCurve = ResolveCurve())
	{
		OutValue = RealCurve->Eval(ProspectiveTime);
		return true;
	}
	return false;
}
