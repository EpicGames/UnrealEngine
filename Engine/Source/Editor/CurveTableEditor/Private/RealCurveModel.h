// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CoreTypes.h"
#include "CurveEditorTypes.h"
#include "CurveModel.h"
#include "CurveTableEditorHandle.h"
#include "Misc/OptionalFwd.h"
#include "Templates/Tuple.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FCurveEditor;
class UObject;
struct FCurveEditorScreenSpace;
struct FKeyAttributes;
struct FKeyDrawInfo;
struct FKeyHandle;
struct FKeyPosition;
struct FRealCurve;

/*
 * RealCurveModel implements the FCurveModel interface for FRealCurves which allows
 * users to edit FRealCurves in the CurveEditor
 */

class FRealCurveModel : public FCurveModel
{
  public:
	static ECurveEditorViewID EventView;

	explicit FRealCurveModel(FCurveTableEditorHandle InRealCurveHandle, UObject* InOwner);

	// FCurveModel interface
	virtual const void* GetCurve() const override;

	virtual void Modify() override;

	virtual void DrawCurve(const FCurveEditor& CurveEditor, const FCurveEditorScreenSpace& ScreenSpace, TArray<TTuple<double, double>>& InterpolatingPoints) const override;
	virtual void GetKeys(double MinTime, double MaxTime, double MinValue, double MaxValue, TArray<FKeyHandle>& OutKeyHandles) const override;
	virtual void AddKeys(TArrayView<const FKeyPosition> InKeyPositions, TArrayView<const FKeyAttributes> InAttributes, TArrayView<TOptional<FKeyHandle>>* OutKeyHandles) override;
	virtual void RemoveKeys(TArrayView<const FKeyHandle> InKey, double InCurrentTime) override;

	virtual void GetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyPosition> OutKeyPositions) const override;
	virtual void SetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyPosition> InKeyPositions, EPropertyChangeType::Type ChangeType) override;
	virtual void GetKeyDrawInfo(ECurvePointType PointType, const FKeyHandle InKeyHandle, FKeyDrawInfo& OutDrawInfo) const override;

	virtual void GetKeyAttributes(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyAttributes> OutAttributes) const override;

	virtual void GetTimeRange(double& MinTime, double& MaxTime) const override;
	virtual void GetValueRange(double& MinValue, double& MaxValue) const override;

	virtual int32 GetNumKeys() const override;
	virtual void GetNeighboringKeys(const FKeyHandle InKeyHandle, TOptional<FKeyHandle>& OutPreviousKeyHandle, TOptional<FKeyHandle>& OutNextKeyHandle) const override;

	virtual bool Evaluate(double ProspectiveTime, double& OutValue) const override;

	virtual bool IsValid() const { return RealCurveHandle.IsValid(); }

  private:

  	const TWeakObjectPtr<> WeakOwner;
	const FCurveTableEditorHandle RealCurveHandle;
	
	FRealCurve* ResolveCurve(bool bEnsure = true) const
	{
		FRealCurve* Result = RealCurveHandle.GetCurve();
		ensure(!bEnsure || Result);
		return Result;
	}
};