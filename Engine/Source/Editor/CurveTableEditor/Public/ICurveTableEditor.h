// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/AssetEditorToolkit.h"


/** CurveTable Editor public interface */
class ICurveTableEditor : public FAssetEditorToolkit
{

public:

	// ...

};


